// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMergingLibrary.h"
#include "Engine/SkeletalMeshSocket.h"
#include "Engine/SkeletalMesh.h"
#include "Algo/Accumulate.h"
#include "Animation/BlendProfile.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletalMergingLibrary)

IMPLEMENT_MODULE(IModuleInterface, SkeletalMerging);

DEFINE_LOG_CATEGORY(LogSkeletalMeshMerge);

namespace UE
{
	namespace SkeletonMerging
	{
		// Helper structure to merge bone hierarchies together and populate a FReferenceSkeleton with the result(s)
		struct FMergedBoneHierarchy
		{
			FMergedBoneHierarchy(uint32 NumExpectedBones)
			{
				BoneNamePose.Reserve(NumExpectedBones);
				PathToBoneNames.Reserve(NumExpectedBones);
				PathHashToBoneNames.Reserve(NumExpectedBones);
			}

			void AddBone(const FName& BoneName, const FTransform& ReferencePose, uint32 PathHash)
			{
				// Store reference transform according to bone name hash
				BoneNamePose.Add(BoneName, ReferencePose);

				// Add bone as child to parent path
				PathHashToBoneNames.FindOrAdd(PathHash).Add(BoneName);

				// Append bone hash to parent path and store 
				const uint32 BoneHash = GetTypeHash(BoneName);
				PathToBoneNames.Add(BoneName, HashCombine(PathHash, BoneHash));
			}

			void PopulateSkeleton(FReferenceSkeletonModifier& SkeletonModifier)
			{
				const uint32 Zero = 0;
				const uint32 RootParentHash = HashCombine(Zero, Zero);

				// Root bone is always parented to 0 hash data entry, so we expect a single root-bone (child)
				const TSet<FName>& ChildBoneNames = GetChildBonesForPath(RootParentHash);
				ensure(ChildBoneNames.Num() == 1);
				const FName RootBoneName = ChildBoneNames.Array()[0];

				// Add root-bone and traverse data to populate child hierarchies
				const FMeshBoneInfo BoneInfo(RootBoneName, RootBoneName.ToString(), INDEX_NONE);
				SkeletonModifier.Add(BoneInfo, GetReferencePose(RootBoneName));

				RecursiveAddBones(SkeletonModifier, RootBoneName);
			}
		protected:			
			const FTransform& GetReferencePose(const FName& InName) const { return BoneNamePose.FindChecked(InName); }
			uint32 GetBonePathHash(const FName& InName) const { return PathToBoneNames.FindChecked(InName); }
			const TSet<FName>* FindChildBonesForPath(uint32 InPath) const { return PathHashToBoneNames.Find(InPath); }
			const TSet<FName>& GetChildBonesForPath(uint32 InPath) const { return PathHashToBoneNames.FindChecked(InPath); }
			
			void RecursiveAddBones(FReferenceSkeletonModifier& SkeletonModifier, const FName ParentBoneName)
			{
				const uint32 PathHash = GetBonePathHash(ParentBoneName);
				if (const TSet<FName>* BoneNamesPtr = FindChildBonesForPath(PathHash))
				{
					for (const FName& ChildBoneName : *BoneNamesPtr)
					{
						FMeshBoneInfo BoneInfo(ChildBoneName, ChildBoneName.ToString(), SkeletonModifier.FindBoneIndex(ParentBoneName));
						SkeletonModifier.Add(BoneInfo, GetReferencePose(ChildBoneName));
						RecursiveAddBones(SkeletonModifier, ChildBoneName);
					}
				}
			}

		private:			
			// Reference pose transform for given bone name
			TMap<FName, FTransform> BoneNamePose;
			// Accumulated hierarchy hash from bone to root bone
			TMap<FName, uint32> PathToBoneNames;
			// Set of child bones for given hierarchy hash
			TMap<uint32, TSet<FName>> PathHashToBoneNames;
		};
	}
}

USkeleton* USkeletalMergingLibrary::MergeSkeletons(const FSkeletonMergeParams& Params)
{
	// List of unique skeletons generated from input parameters
	TArray<TObjectPtr<USkeleton>> ToMergeSkeletons;
	for ( TObjectPtr<USkeleton> SkeletonPtr : Params.SkeletonsToMerge)
	{
		ToMergeSkeletons.AddUnique(SkeletonPtr);
	}

	// Ensure we have at least one valid Skeleton to merge
	const int32 NumberOfSkeletons = ToMergeSkeletons.Num();
	if (NumberOfSkeletons == 0)
	{
		return nullptr;
	}

	// Calculate potential total number of bones, used for pre-allocating data arrays
	const int32 TotalPossibleBones = Algo::Accumulate(ToMergeSkeletons, 0, [](int32 Sum, const USkeleton* Skeleton)
	{
		return Sum + Skeleton->GetReferenceSkeleton().GetRawBoneNum();
	});

	// Ensure a valid skeleton (number of bones) will be generated
	if (TotalPossibleBones == 0)
	{
		return nullptr;
	}

	UE::SkeletonMerging::FMergedBoneHierarchy MergedBoneHierarchy(TotalPossibleBones);
	
	// Accumulated hierarchy hash from parent-bone to root bone
	TMap<FName, uint32> BoneNamesToPathHash;
	BoneNamesToPathHash.Reserve(TotalPossibleBones);

	// Bone name to bone pose 
	TMap<FName, FTransform> BoneNamesToBonePose;
	BoneNamesToBonePose.Reserve(TotalPossibleBones);

	// Combined bone and socket name hash
	TMap<uint32, TObjectPtr<USkeletalMeshSocket>> HashToSockets;
	// Combined from and to-bone name hash
	TMap<uint32, const FVirtualBone*> HashToVirtualBones;

	TMap<FName, const FCurveMetaData*> UniqueCurveNames;
	TMap<FName, TSet<FName>> GroupToSlotNames;
	TMap<FName, TArray<const UBlendProfile*>> UniqueBlendProfiles;

	bool bMergeSkeletonsFailed = false;

	for (int32 SkeletonIndex = 0; SkeletonIndex < NumberOfSkeletons; ++SkeletonIndex)
	{
		const USkeleton* Skeleton = ToMergeSkeletons[SkeletonIndex];
		const FReferenceSkeleton& ReferenceSkeleton = Skeleton->GetReferenceSkeleton();
		const TArray<FMeshBoneInfo>& Bones = ReferenceSkeleton.GetRawRefBoneInfo();		
		const TArray<FTransform>& BonePoses = ReferenceSkeleton.GetRawRefBonePose();


		bool bConflictivePoseFound = false;

		const int32 NumBones = Bones.Num();
		for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
		{
			const FMeshBoneInfo& Bone = Bones[BoneIndex];

			// Retrieve parent bone name and respective hash, root-bone is assumed to have a parent hash of 0
			const FName ParentName = Bone.ParentIndex != INDEX_NONE ? Bones[Bone.ParentIndex].Name : NAME_None;
			const uint32 ParentHash = Bone.ParentIndex != INDEX_NONE ? GetTypeHash(ParentName) : 0;

			// Look-up the path-hash from root to the parent bone
			const uint32* ParentPath = BoneNamesToPathHash.Find(ParentName);
			const uint32 ParentPathHash = ParentPath ? *ParentPath : 0;

			// Append parent hash to path to give full path hash to current bone
			const uint32 BonePathHash = HashCombine(ParentPathHash, ParentHash);

			if (Params.bCheckSkeletonsCompatibility)
			{
				// Check if the bone exists in the hierarchy 
				if (const uint32* ExistingPath = BoneNamesToPathHash.Find(Bone.Name))
				{
					const uint32 ExistingPathHash = *ExistingPath;

					// If the hash differs from the existing one it means skeletons are incompatible
					if (ExistingPathHash != BonePathHash)
					{
						UE_LOGF(LogSkeletalMeshMerge, Error, "Failed to merge skeletons. Skeleton %ls has an invalid bone chain.", *Skeleton->GetName());
						bMergeSkeletonsFailed = true;
						break;
					}

					// Bone poses will be overwritten, check if they are the same
					if (!bConflictivePoseFound && !BoneNamesToBonePose[Bone.Name].Equals(BonePoses[BoneIndex]))
					{
						UE_LOGF(LogSkeletalMeshMerge, Warning, "Skeleton %ls has a different reference pose, reference pose will be overwritten.", *Skeleton->GetName());
						bConflictivePoseFound = true;
					}
				}

				BoneNamesToBonePose.Add(Bone.Name, BonePoses[BoneIndex]);
			}
			
			// Add path hash to current bone
			BoneNamesToPathHash.Add(Bone.Name, BonePathHash);

			// Add bone to hierarchy
			MergedBoneHierarchy.AddBone(Bone.Name, BonePoses[BoneIndex], BonePathHash);
			
		}

		if (Params.bCheckSkeletonsCompatibility && bMergeSkeletonsFailed)
		{
			continue;
		}

		if (Params.bMergeSockets)
		{
			for (const TObjectPtr<USkeletalMeshSocket>& Socket : Skeleton->Sockets)
			{
				const uint32 Hash = HashCombine(GetTypeHash(Socket->SocketName), GetTypeHash(Socket->BoneName));
				HashToSockets.Add(Hash, Socket);
			}
		}

		if (Params.bMergeVirtualBones)
		{
			const TArray<FVirtualBone>& VirtualBones = Skeleton->GetVirtualBones();		
	        for (const FVirtualBone& VB : VirtualBones)
	        {
	            const uint32 Hash = HashCombine(GetTypeHash(VB.SourceBoneName), GetTypeHash(VB.TargetBoneName));
	            HashToVirtualBones.Add(Hash, &VB);
	        }
		}		

		if (Params.bMergeCurveNames)
		{
			Skeleton->ForEachCurveMetaData([&UniqueCurveNames](FName InCurveName, const FCurveMetaData& InCurveMetaData)
			{
				UniqueCurveNames.FindOrAdd(InCurveName, &InCurveMetaData);
			});
		}

		if (Params.bMergeAnimSlotGroups)
		{
			const TArray<FAnimSlotGroup>& SlotGroups = Skeleton->GetSlotGroups();
			for (const FAnimSlotGroup& AnimSlotGroup : SlotGroups)
			{
				GroupToSlotNames.FindOrAdd(AnimSlotGroup.GroupName).Append(AnimSlotGroup.SlotNames);
			}
		}
		
		if (Params.bMergeBlendProfiles)
		{
			for ( const TObjectPtr<UBlendProfile>& BlendProfile : Skeleton->BlendProfiles)
			{
				UniqueBlendProfiles.FindOrAdd(BlendProfile->GetFName()).Add(BlendProfile.Get());
			}			
		}		
	}

	if (bMergeSkeletonsFailed)
	{
		UE_LOGF(LogSkeletalMeshMerge, Error, "Failed to merge skeletons. One or more skeletons with invalid parent chains were found.");
		return nullptr;
	}

	USkeleton* GeneratedSkeleton = NewObject<USkeleton>();

	// Generate bone hierarchy
	{		
		FReferenceSkeletonModifier Modifier(GeneratedSkeleton);
		MergedBoneHierarchy.PopulateSkeleton(Modifier);
	}
	
	// Merge sockets
	if (Params.bMergeSockets)
	{
		TArray<TObjectPtr<USkeletalMeshSocket>> Sockets;
		HashToSockets.GenerateValueArray(Sockets);
		AddSockets(GeneratedSkeleton, Sockets);
	}
	
	// Merge virtual bones
	if (Params.bMergeVirtualBones)
	{
		TArray<const FVirtualBone*> VirtualBones;
		HashToVirtualBones.GenerateValueArray(VirtualBones);
		AddVirtualBones(GeneratedSkeleton, VirtualBones);
	}

	// Merge Curve / track mappings	
	if (Params.bMergeCurveNames)
	{
		AddCurveNames(GeneratedSkeleton, UniqueCurveNames);
	}

	// Merge blend profiles
	if (Params.bMergeBlendProfiles)
	{
		AddBlendProfiles(GeneratedSkeleton, UniqueBlendProfiles);
	}

	// Merge SlotGroups
	if (Params.bMergeAnimSlotGroups)
	{
		AddAnimationSlotGroups(GeneratedSkeleton, GroupToSlotNames);
	}
		
	return GeneratedSkeleton;
}

void USkeletalMergingLibrary::AddSockets(USkeleton* InSkeleton, const TArray<TObjectPtr<USkeletalMeshSocket>>& InSockets)
{
	for (const TObjectPtr<USkeletalMeshSocket>& MergeSocket : InSockets)
	{
		USkeletalMeshSocket* NewSocket = NewObject<USkeletalMeshSocket>(InSkeleton);
		if (NewSocket != nullptr)
		{
			InSkeleton->Sockets.Add(NewSocket);

			// Copy over all socket information
			NewSocket->SocketName = MergeSocket->SocketName;
			NewSocket->BoneName = MergeSocket->BoneName;
			NewSocket->RelativeLocation = MergeSocket->RelativeLocation;
			NewSocket->RelativeRotation = MergeSocket->RelativeRotation;
			NewSocket->RelativeScale = MergeSocket->RelativeScale;
			NewSocket->bForceAlwaysAnimated = MergeSocket->bForceAlwaysAnimated;
		}
	}
}

void USkeletalMergingLibrary::AddVirtualBones(USkeleton* InSkeleton, const TArray<const FVirtualBone*>& InVirtualBones)
{
	for (const FVirtualBone* VirtualBone : InVirtualBones)
	{
		InSkeleton->AddNewNamedVirtualBone(VirtualBone->SourceBoneName, VirtualBone->TargetBoneName, VirtualBone->VirtualBoneName);
	}	
}

void USkeletalMergingLibrary::AddCurveNames(USkeleton* InSkeleton, const TMap<FName, const FCurveMetaData*>& InCurves)
{
	for(const TPair<FName, const FCurveMetaData*>& CurveMetaDataPair : InCurves)
	{
		if (CurveMetaDataPair.Value)
		{
			InSkeleton->AddCurveMetaData(CurveMetaDataPair.Key, false);

			FCurveMetaData& SkeletonCurveMetaData = *InSkeleton->GetCurveMetaData(CurveMetaDataPair.Key);
			SkeletonCurveMetaData = *CurveMetaDataPair.Value;
			for (FBoneReference& BoneReference : SkeletonCurveMetaData.LinkedBones)
			{
				BoneReference.Initialize(InSkeleton);
			}
		}
	}
}

void USkeletalMergingLibrary::AddBlendProfiles(USkeleton* InSkeleton, const TMap<FName, TArray<const UBlendProfile*>>& InBlendProfiles)
{
	for (const TPair<FName, TArray<const UBlendProfile*>>& BlendProfilesPair : InBlendProfiles)
	{
		const TArray<const UBlendProfile*>& BlendProfiles = BlendProfilesPair.Value;
		UBlendProfile* MergedBlendProfile = InSkeleton->CreateNewBlendProfile(BlendProfilesPair.Key);
			
		for (int32 ProfileIndex = 0; ProfileIndex < BlendProfiles.Num(); ++ProfileIndex)
		{
			const UBlendProfile* Profile = BlendProfiles[ProfileIndex];				
			MergedBlendProfile->Mode = ProfileIndex == 0 ? Profile->Mode : MergedBlendProfile->Mode;

			// Mismatch in terms of blend profile type
			ensure(MergedBlendProfile->Mode == Profile->Mode);

			for (const FBlendProfileBoneEntry& Entry : Profile->ProfileEntries)
			{
				// Overlapping bone entries
				ensure(!MergedBlendProfile->ProfileEntries.ContainsByPredicate([Entry](const FBlendProfileBoneEntry& InEntry)
				{
					return InEntry.BoneReference.BoneName == Entry.BoneReference.BoneName;
				}));

				MergedBlendProfile->SetBoneBlendScale(Entry.BoneReference.BoneName, Entry.BlendScale, false, true);
			}
		}
	}
}

void USkeletalMergingLibrary::AddAnimationSlotGroups(USkeleton* InSkeleton, const TMap<FName, TSet<FName>>& InSlotGroupsNames)
{
	for (const TPair<FName, TSet<FName>>& SlotGroupNamePair : InSlotGroupsNames)
	{
		InSkeleton->AddSlotGroupName(SlotGroupNamePair.Key);
		for (const FName& SlotName : SlotGroupNamePair.Value)
		{
			InSkeleton->SetSlotGroupName(SlotName, SlotGroupNamePair.Key);				
		}
	}
}

USkeletalMesh* USkeletalMergingLibrary::MergeMeshes(const FSkeletalMeshMergeParams& Params)
{
	TArray<USkeletalMesh*> MeshesToMergeCopy = Params.MeshesToMerge;
	MeshesToMergeCopy.RemoveAll([](USkeletalMesh* InMesh)
	{
		return InMesh == nullptr;
	});

	if (MeshesToMergeCopy.Num() <= 1)
	{
		UE_LOGF(LogTemp, Warning, "Must provide multiple valid Skeletal Meshes in order to perform a merge.");
		return nullptr;
	}

	EMeshBufferAccess BufferAccess = Params.bNeedsCpuAccess ?
										EMeshBufferAccess::ForceCPUAndGPU :
										EMeshBufferAccess::Default;
	
	bool bRunDuplicateCheck = false;
	USkeletalMesh* BaseMesh = NewObject<USkeletalMesh>();
	
	if (Params.Skeleton && Params.bSkeletonBefore)
	{
		BaseMesh->SetSkeleton(Params.Skeleton);
		bRunDuplicateCheck = true;

		for (USkeletalMeshSocket* Socket : BaseMesh->GetMeshOnlySocketList())
		{
			if (Socket)
			{
				UE_LOGF(LogTemp, Warning, "SkelMeshSocket: %ls", *(Socket->SocketName.ToString()));
			}
		}

		for (USkeletalMeshSocket* Socket : BaseMesh->GetSkeleton()->Sockets)
		{
			if (Socket)
			{
				UE_LOGF(LogTemp, Warning, "SkelSocket: %ls", *(Socket->SocketName.ToString()));
			}
		}
	}

	FSkelMeshMergeUVTransformMapping Mapping;
	Mapping.UVTransformsPerMesh = Params.UVTransformsPerMesh;
	FSkeletalMeshMerge Merger(BaseMesh, MeshesToMergeCopy, Params.MeshSectionMappings, Params.StripTopLODS, BufferAccess, &Mapping);
	if (!Merger.DoMerge())
	{
		UE_LOGF(LogTemp, Warning, "Merge failed!");
		return nullptr;
	}

	if (Params.Skeleton && !Params.bSkeletonBefore)
	{
		BaseMesh->SetSkeleton(Params.Skeleton);
	}

	if (bRunDuplicateCheck)
	{
		TArray<FName> SkelMeshSockets;
		TArray<FName> SkelSockets;

		for (USkeletalMeshSocket* Socket : BaseMesh->GetMeshOnlySocketList())
		{
			if (Socket)
			{
				SkelMeshSockets.Add(Socket->GetFName());
				UE_LOGF(LogTemp, Warning, "SkelMeshSocket: %ls", *(Socket->SocketName.ToString()));
			}
		}

		for (USkeletalMeshSocket* Socket : BaseMesh->GetSkeleton()->Sockets)
		{
			if (Socket)
			{
				SkelSockets.Add(Socket->GetFName());
				UE_LOGF(LogTemp, Warning, "SkelSocket: %ls", *(Socket->SocketName.ToString()));
			}
		}

		TSet<FName> UniqueSkelMeshSockets;
		TSet<FName> UniqueSkelSockets;

		UniqueSkelMeshSockets.Append(SkelMeshSockets);
		UniqueSkelSockets.Append(SkelSockets);

		int32 Total = SkelSockets.Num() + SkelMeshSockets.Num();
		int32 UniqueTotal = UniqueSkelMeshSockets.Num() + UniqueSkelSockets.Num();

		UE_LOGF(LogTemp, Warning, "SkelMeshSocketCount: %d | SkelSocketCount: %d | Combined: %d", SkelMeshSockets.Num(), SkelSockets.Num(), Total);
		UE_LOGF(LogTemp, Warning, "SkelMeshSocketCount: %d | SkelSocketCount: %d | Combined: %d", UniqueSkelMeshSockets.Num(), UniqueSkelSockets.Num(), UniqueTotal);
		UE_LOGF(LogTemp, Warning, "Found Duplicates: %ls", *((Total != UniqueTotal) ? FString("True") : FString("False")));
	}

	return BaseMesh;
}

