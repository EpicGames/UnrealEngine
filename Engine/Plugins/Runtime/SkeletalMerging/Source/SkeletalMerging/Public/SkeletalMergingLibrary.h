// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SkeletalMeshMerge.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "SkeletalMergingLibrary.generated.h"

#define UE_API SKELETALMERGING_API

DECLARE_LOG_CATEGORY_EXTERN(LogSkeletalMeshMerge, Log, All);

/**
* Struct containing all parameters used to perform a Skeletal Mesh merge.
*/
USTRUCT(BlueprintType)
struct FSkeletalMeshMergeParams
{
	GENERATED_USTRUCT_BODY()

	FSkeletalMeshMergeParams()
	{
		MeshSectionMappings = TArray<FSkelMeshMergeSectionMapping>();
		UVTransformsPerMesh = TArray<FSkelMeshMergeMeshUVTransforms>();

		StripTopLODS = 0;
		bNeedsCpuAccess = false;
		bSkeletonBefore = false;
		Skeleton = nullptr;
	}

	// An optional array to map sections from the source meshes to merged section entries
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SkeletalMeshMerge)
	TArray<FSkelMeshMergeSectionMapping> MeshSectionMappings;

	// An optional array to transform the UVs in each mesh
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SkeletalMeshMerge)
	TArray<FSkelMeshMergeMeshUVTransforms> UVTransformsPerMesh;

	// The list of skeletal meshes to merge.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SkeletalMeshMerge)
	TArray<TObjectPtr<USkeletalMesh>> MeshesToMerge;

	// The number of high LODs to remove from input meshes
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SkeletalMeshMerge)
	int32 StripTopLODS;

	// Whether or not the resulting mesh needs to be accessed by the CPU for any reason (e.g. for spawning particle effects).
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SkeletalMeshMerge)
	uint32 bNeedsCpuAccess:1;

	// Update skeleton before merge. Otherwise, update after.
	// Skeleton must also be provided.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SkeletalMeshMerge)
	uint32 bSkeletonBefore:1;

	// Skeleton that will be used for the merged mesh.
	// Leave empty if the generated skeleton is OK.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SkeletalMeshMerge)
	TObjectPtr<USkeleton> Skeleton;
};

/**
* Struct containing all parameters used to perform a Skeleton merge.
*/
USTRUCT(BlueprintType)
struct FSkeletonMergeParams
{
	GENERATED_USTRUCT_BODY()

	// The list of skeletons to merge.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SkeletonMerge)
	TArray<TObjectPtr<USkeleton>> SkeletonsToMerge;

	// Whether or not to include Sockets when merging the Skeletons
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SkeletonMerge)
	bool bMergeSockets = true;

	// Whether or not to include Virtual Bones when merging the Skeletons
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SkeletonMerge)
	bool bMergeVirtualBones = true;

	// Whether or not to include (Animation) Curve names when merging the Skeletons
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SkeletonMerge)
	bool bMergeCurveNames = true;

	// Whether or not to include Blend Profiles when merging the Skeletons
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SkeletonMerge)
	bool bMergeBlendProfiles = true;

	// Whether or not to include Animation Slot Group (names) when merging the Skeletons
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SkeletonMerge)
	bool bMergeAnimSlotGroups = true;

	// Whether or not to check if there are invalid parent chains or shared bones with different reference transforms
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SkeletonMerge)
	bool bCheckSkeletonsCompatibility = false;
};

/**
* Component that can be used to perform Skeletal Mesh merges from Blueprints.
*/
UCLASS(MinimalAPI,  ClassGroup=(Custom) )
class USkeletalMergingLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:	
	/**
	* Merges the given meshes into a single mesh.
	* @return The merged mesh (will be invalid if the merge failed).
	*/
	UFUNCTION(BlueprintCallable, Category="Mesh Merge", meta=(UnsafeDuringActorConstruction="true"))
	static UE_API class USkeletalMesh* MergeMeshes(const FSkeletalMeshMergeParams& Params);
	
	/**
	* Merges the skeletons for the provided meshes into a single skeleton.
	* @return The merged Skeleton
	*/
	UFUNCTION(BlueprintCallable, Category="Skeleton Merge", meta=(UnsafeDuringActorConstruction="true"))
	static UE_API class USkeleton* MergeSkeletons(const FSkeletonMergeParams& Params);

protected:
	static UE_API void AddSockets(USkeleton* InSkeleton, const TArray<TObjectPtr<class USkeletalMeshSocket>>& InSockets);
	static UE_API void AddVirtualBones(USkeleton* InSkeleton, const TArray<const struct FVirtualBone*>& InVirtualBones);
	static UE_API void AddCurveNames(USkeleton* InSkeleton, const TMap<FName, const struct FCurveMetaData*>& InCurves);
	static UE_API void AddBlendProfiles(USkeleton* InSkeleton, const TMap<FName, TArray<const class UBlendProfile*>>& InBlendProfiles);
	static UE_API void AddAnimationSlotGroups(USkeleton* InSkeleton, const TMap<FName, TSet<FName>>& InSlotGroupsNames);
};

#undef UE_API
